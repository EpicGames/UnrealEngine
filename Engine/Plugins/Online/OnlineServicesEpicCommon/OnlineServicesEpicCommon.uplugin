{
	"FileVersion": 3,

	"FriendlyName": "Online Services Epic Common",
	"Version": 1,
	"VersionName": "1.0",
	"Description": "Common Online Services implementation with shared funcionality between Epic Account Services and Epic Game Services.",
	"Category": "Online Platform",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"EnabledByDefault": false,

	"Modules": [
		{
			"Name": "OnlineServicesEpicCommon",
			"Type": "Runtime",
			"LoadingPhase": "PostConfigInit",
			"PlatformAllowList": [
				"Android",
				"IOS",
				"Linux",
				"LinuxArm64",
				"Mac",
				"Win64"
			]
		}
	],
	"Plugins": [
		{
			"Name": "OnlineServices",
			"Enabled": true
		},
		{
			"Name": "EOSShared",
			"Enabled": true
		}
	]
}