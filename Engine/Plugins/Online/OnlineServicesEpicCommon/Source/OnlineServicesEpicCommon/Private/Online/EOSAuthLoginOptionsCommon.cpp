// Copyright Epic Games, Inc. All Rights Reserved.

#include "Online/EOSAuthLoginOptionsCommon.h"

#include "EOSShared.h"

namespace UE::Online {

namespace CompatibilityLoginCredentialsType
{
	const FName Password = TEXT("epic");
	const FName Developer = TEXT("dev_tool");
}

const FEOSAuthTranslationTraits* FEOSAuthLoginOptionsCommon::GetLoginTranslatorTraits(FName Name)
{
	static const TMap<FName, FEOSAuthTranslationTraits> SupportedLoginTranslatorTraits = {
		{ LoginCredentialsType::Password, { EOS_ELoginCredentialType::EOS_LCT_Password, EEOSAuthTranslationFlags::SetId | EEOSAuthTranslationFlags::SetTokenFromString } },
		{ LoginCredentialsType::ExchangeCode, { EOS_ELoginCredentialType::EOS_LCT_ExchangeCode, EEOSAuthTranslationFlags::SetTokenFromString } },
		{ LoginCredentialsType::PersistentAuth, { EOS_ELoginCredentialType::EOS_LCT_PersistentAuth, EEOSAuthTranslationFlags::None } },
		{ LoginCredentialsType::Developer, { EOS_ELoginCredentialType::EOS_LCT_Developer, EEOSAuthTranslationFlags::SetId | EEOSAuthTranslationFlags::SetTokenFromString } },
		{ LoginCredentialsType::RefreshToken, { EOS_ELoginCredentialType::EOS_LCT_RefreshToken, EEOSAuthTranslationFlags::SetTokenFromString } },
		{ LoginCredentialsType::AccountPortal, { EOS_ELoginCredentialType::EOS_LCT_AccountPortal, EEOSAuthTranslationFlags::SetId | EEOSAuthTranslationFlags::SetTokenFromString } },
		{ LoginCredentialsType::ExternalAuth, { EOS_ELoginCredentialType::EOS_LCT_ExternalAuth, EEOSAuthTranslationFlags::SetTokenFromExternalAuth } },
		{ CompatibilityLoginCredentialsType::Password, { EOS_ELoginCredentialType::EOS_LCT_Password, EEOSAuthTranslationFlags::SetId | EEOSAuthTranslationFlags::SetTokenFromString } },
		{ CompatibilityLoginCredentialsType::Developer, { EOS_ELoginCredentialType::EOS_LCT_Developer, EEOSAuthTranslationFlags::SetId | EEOSAuthTranslationFlags::SetTokenFromString } },
	};

	return SupportedLoginTranslatorTraits.Find(Name);
}

const FEOSExternalAuthTranslationTraits* FEOSAuthLoginOptionsCommon::GetExternalAuthTranslationTraits(FName ExternalAuthType)
{
	static const TMap<FName, FEOSExternalAuthTranslationTraits> SupportedExternalAuthTraits = {
		{ ExternalLoginType::Epic, { EOS_EExternalCredentialType::EOS_ECT_EPIC, EOS_ELinkAccountFlags::EOS_LA_NoFlags } },
		{ ExternalLoginType::SteamSessionTicket, { EOS_EExternalCredentialType::EOS_ECT_STEAM_SESSION_TICKET, EOS_ELinkAccountFlags::EOS_LA_NoFlags } },
		{ ExternalLoginType::PsnIdToken, { EOS_EExternalCredentialType::EOS_ECT_PSN_ID_TOKEN, EOS_ELinkAccountFlags::EOS_LA_NoFlags } },
		{ ExternalLoginType::XblXstsToken, { EOS_EExternalCredentialType::EOS_ECT_XBL_XSTS_TOKEN, EOS_ELinkAccountFlags::EOS_LA_NoFlags } },
		{ ExternalLoginType::DiscordAccessToken, { EOS_EExternalCredentialType::EOS_ECT_DISCORD_ACCESS_TOKEN, EOS_ELinkAccountFlags::EOS_LA_NoFlags } },
		{ ExternalLoginType::GogSessionTicket, { EOS_EExternalCredentialType::EOS_ECT_GOG_SESSION_TICKET, EOS_ELinkAccountFlags::EOS_LA_NoFlags } },
		{ ExternalLoginType::NintendoIdToken, { EOS_EExternalCredentialType::EOS_ECT_NINTENDO_ID_TOKEN, EOS_ELinkAccountFlags::EOS_LA_NoFlags } },
		{ ExternalLoginType::NintendoNsaIdToken, { EOS_EExternalCredentialType::EOS_ECT_NINTENDO_NSA_ID_TOKEN, EOS_ELinkAccountFlags::EOS_LA_NintendoNsaId } },
		{ ExternalLoginType::UplayAccessToken, { EOS_EExternalCredentialType::EOS_ECT_UPLAY_ACCESS_TOKEN, EOS_ELinkAccountFlags::EOS_LA_NoFlags } },
		{ ExternalLoginType::OpenIdAccessToken, { EOS_EExternalCredentialType::EOS_ECT_OPENID_ACCESS_TOKEN, EOS_ELinkAccountFlags::EOS_LA_NoFlags } },
		{ ExternalLoginType::DeviceIdAccessToken, { EOS_EExternalCredentialType::EOS_ECT_DEVICEID_ACCESS_TOKEN, EOS_ELinkAccountFlags::EOS_LA_NoFlags } },
		{ ExternalLoginType::AppleIdToken, { EOS_EExternalCredentialType::EOS_ECT_APPLE_ID_TOKEN, EOS_ELinkAccountFlags::EOS_LA_NoFlags } },
		{ ExternalLoginType::GoogleIdToken, { EOS_EExternalCredentialType::EOS_ECT_GOOGLE_ID_TOKEN, EOS_ELinkAccountFlags::EOS_LA_NoFlags } },
		{ ExternalLoginType::OculusUserIdNonce, { EOS_EExternalCredentialType::EOS_ECT_OCULUS_USERID_NONCE, EOS_ELinkAccountFlags::EOS_LA_NoFlags } },
		{ ExternalLoginType::ItchioJwt, { EOS_EExternalCredentialType::EOS_ECT_ITCHIO_JWT, EOS_ELinkAccountFlags::EOS_LA_NoFlags } },
		{ ExternalLoginType::ItchioKey, { EOS_EExternalCredentialType::EOS_ECT_ITCHIO_KEY, EOS_ELinkAccountFlags::EOS_LA_NoFlags } },
		{ ExternalLoginType::EpicIdToken, { EOS_EExternalCredentialType::EOS_ECT_EPIC_ID_TOKEN, EOS_ELinkAccountFlags::EOS_LA_NoFlags } },
		{ ExternalLoginType::AmazonAccessToken, { EOS_EExternalCredentialType::EOS_ECT_AMAZON_ACCESS_TOKEN, EOS_ELinkAccountFlags::EOS_LA_NoFlags } },
	};

	return SupportedExternalAuthTraits.Find(ExternalAuthType);
}

FEOSAuthLoginOptionsCommon::FEOSAuthLoginOptionsCommon(FEOSAuthLoginOptionsCommon&& Other)
{
	*this = MoveTemp(Other);
}

FEOSAuthLoginOptionsCommon& FEOSAuthLoginOptionsCommon::operator=(FEOSAuthLoginOptionsCommon&& Other)
{
	CredentialsData = Other.CredentialsData;

	// Pointer fixup.
	if (CredentialsData.Id)
	{
		IdUtf8 = MoveTemp(Other.IdUtf8);
		CredentialsData.Id = IdUtf8.GetData();
	}
	if (CredentialsData.Token)
	{
		TokenUtf8 = MoveTemp(Other.TokenUtf8);
		CredentialsData.Token = TokenUtf8.GetData();
		LinkAccountFlags = Other.LinkAccountFlags;
	}
	if (CredentialsData.SystemAuthCredentialsOptions)
	{
		// todo
	}

	Credentials = &CredentialsData;
	ApiVersion = Other.ApiVersion;
	ScopeFlags = Other.ScopeFlags;

	Other.Credentials = nullptr;
	return *this;
}

FEOSAuthLoginOptionsCommon::FEOSAuthLoginOptionsCommon()
{
	// EOS_Auth_LoginOptions init
	UE_EOS_CHECK_API_MISMATCH(EOS_AUTH_LOGIN_API_LATEST, 3);
	ApiVersion = 3;
	Credentials = &CredentialsData;
	ScopeFlags = EOS_EAuthScopeFlags::EOS_AS_NoFlags;
	LoginFlags = 0;

	// EOS_Auth_Credentials init
	UE_EOS_CHECK_API_MISMATCH(EOS_AUTH_CREDENTIALS_API_LATEST, 4);
	CredentialsData.ApiVersion = 4;
	CredentialsData.Id = nullptr;
	CredentialsData.Token = nullptr;
	CredentialsData.Type = EOS_ELoginCredentialType::EOS_LCT_Password;
	CredentialsData.SystemAuthCredentialsOptions = nullptr;
	CredentialsData.ExternalType = EOS_EExternalCredentialType::EOS_ECT_EPIC;
}
}