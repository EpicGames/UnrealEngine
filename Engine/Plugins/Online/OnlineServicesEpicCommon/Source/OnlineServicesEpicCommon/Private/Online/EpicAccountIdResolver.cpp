// Copyright Epic Games, Inc. All Rights Reserved.

#include "Online/EpicAccountIdResolver.h"

#include "Online/OnlineAsyncOp.h"

namespace UE::Online
{

TFunction<TFuture<FAccountId>(FOnlineAsyncOp& InAsyncOp, const EOS_EpicAccountId EpicAccountId)> IEpicAccountIdResolver::ResolveEpicAccountIdFn()
{
	return [this](FOnlineAsyncOp& InAsyncOp, const EOS_EpicAccountId& EpicAccountId)
	{
		const FAccountId* LocalAccountIdPtr = InAsyncOp.Data.Get<FAccountId>(TEXT("LocalAccountId"));
		if (!ensure(LocalAccountIdPtr))
		{
			return MakeFulfilledPromise<FAccountId>().GetFuture();
		}
		return ResolveAccountId(*LocalAccountIdPtr, EpicAccountId);
	};
}

TFunction<TFuture<TArray<FAccountId>>(FOnlineAsyncOp& InAsyncOp, const TArray<EOS_EpicAccountId>& EpicAccountIds)> IEpicAccountIdResolver::ResolveEpicAccountIdsFn()
{
	return [this](FOnlineAsyncOp& InAsyncOp, const TArray<EOS_EpicAccountId>& EpicAccountIds)
	{
		const FAccountId* LocalAccountIdPtr = InAsyncOp.Data.Get<FAccountId>(TEXT("LocalAccountId"));
		if (!ensure(LocalAccountIdPtr))
		{
			return MakeFulfilledPromise<TArray<FAccountId>>().GetFuture();
		}
		return ResolveAccountIds(*LocalAccountIdPtr, EpicAccountIds);
	};
}

} // namespace UE::Online