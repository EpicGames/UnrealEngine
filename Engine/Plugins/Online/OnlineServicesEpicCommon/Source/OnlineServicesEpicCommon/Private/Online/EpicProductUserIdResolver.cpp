// Copyright Epic Games, Inc. All Rights Reserved.

#include "Online/EpicProductUserIdResolver.h"

#include "Online/OnlineAsyncOp.h"

namespace UE::Online
{

TFunction<TFuture<FAccountId>(FOnlineAsyncOp& InAsyncOp, const EOS_ProductUserId ProductUserId)> IEpicProductUserIdResolver::ResolveProductUserIdFn()
{
	return [this](FOnlineAsyncOp& InAsyncOp, const EOS_ProductUserId& ProductUserId)
	{
		const FAccountId* LocalAccountIdPtr = InAsyncOp.Data.Get<FAccountId>(TEXT("LocalAccountId"));
		if (!ensure(LocalAccountIdPtr))
		{
			return MakeFulfilledPromise<FAccountId>().GetFuture();
		}
		return ResolveAccountId(*LocalAccountIdPtr, ProductUserId);
	};
}

TFunction<TFuture<TArray<FAccountId>>(FOnlineAsyncOp& InAsyncOp, const TArray<EOS_ProductUserId>& ProductUserIds)> IEpicProductUserIdResolver::ResolveProductUserIdsFn()
{
	return [this](FOnlineAsyncOp& InAsyncOp, const TArray<EOS_ProductUserId>& ProductUserIds)
	{
		const FAccountId* LocalAccountIdPtr = InAsyncOp.Data.Get<FAccountId>(TEXT("LocalAccountId"));
		if (!ensure(LocalAccountIdPtr))
		{
			return MakeFulfilledPromise<TArray<FAccountId>>().GetFuture();
		}
		return ResolveAccountIds(*LocalAccountIdPtr, ProductUserIds);
	};
}

} // namespace UE::Online