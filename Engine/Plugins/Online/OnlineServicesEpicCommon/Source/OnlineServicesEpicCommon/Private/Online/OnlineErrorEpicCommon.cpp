// Copyright Epic Games, Inc. All Rights Reserved.

#include "Online/OnlineErrorEpicCommon.h"
#include "Online/OnlineErrorDefinitions.h"

#include "EOSShared.h"

namespace UE::Online::Errors {

class FOnlineErrorDetailsEOS : public IOnlineErrorDetails
{
public:
	static const TSharedRef<const IOnlineErrorDetails, ESPMode::ThreadSafe>& Get();

	virtual FString GetFriendlyErrorCode(const FOnlineError& OnlineError) const override;

	virtual FText GetText(const FOnlineError& OnlineError) const override;
	virtual FString GetLogString(const FOnlineError& OnlineError) const override;
};

const TSharedRef<const IOnlineErrorDetails, ESPMode::ThreadSafe>& FOnlineErrorDetailsEOS::Get()
{
	static TSharedRef<const IOnlineErrorDetails, ESPMode::ThreadSafe> Instance = MakeShared<FOnlineErrorDetailsEOS>();
	return Instance;
}

FString FOnlineErrorDetailsEOS::GetFriendlyErrorCode(const FOnlineError& OnlineError) const
{
	const EOS_EResult EosResult = EOS_EResult(OnlineError.GetValue());
	return LexToString(EosResult);
}

FText FOnlineErrorDetailsEOS::GetText(const FOnlineError& OnlineError) const
{
	const EOS_EResult EosResult = EOS_EResult(OnlineError.GetValue());
	return FText::FromString(LexToString(EosResult));
}

FString FOnlineErrorDetailsEOS::GetLogString(const FOnlineError& OnlineError) const
{
	const EOS_EResult EosResult = EOS_EResult(OnlineError.GetValue());
	return LexToString(EosResult);
}

/** This doesn't wrap every single error in EOS, only the common ones that are strongly related to common errors */
FOnlineError MapCommonEOSError(FOnlineError&& Error, EOS_EResult Result)
{
	switch (Result)
	{
	case EOS_EResult::EOS_Success:				return Errors::Success(MoveTemp(Error));
	case EOS_EResult::EOS_NoConnection:			return Errors::NoConnection(MoveTemp(Error));
	case EOS_EResult::EOS_InvalidCredentials:	return Errors::InvalidCreds(MoveTemp(Error));
	case EOS_EResult::EOS_InvalidUser:			return Errors::InvalidUser(MoveTemp(Error));
	case EOS_EResult::EOS_InvalidAuth:			return Errors::InvalidAuth(MoveTemp(Error));
	case EOS_EResult::EOS_AccessDenied:			return Errors::AccessDenied(MoveTemp(Error));
	case EOS_EResult::EOS_MissingPermissions:	return Errors::AccessDenied(MoveTemp(Error));
	case EOS_EResult::EOS_TooManyRequests:		return Errors::TooManyRequests(MoveTemp(Error));
	case EOS_EResult::EOS_AlreadyPending:		return Errors::AlreadyPending(MoveTemp(Error));
	case EOS_EResult::EOS_InvalidParameters:	return Errors::InvalidParams(MoveTemp(Error));
	case EOS_EResult::EOS_InvalidRequest:		return Errors::InvalidParams(MoveTemp(Error));
	case EOS_EResult::EOS_UnrecognizedResponse: return Errors::InvalidResults(MoveTemp(Error));
	case EOS_EResult::EOS_IncompatibleVersion:	return Errors::IncompatibleVersion(MoveTemp(Error));
	case EOS_EResult::EOS_NotConfigured:		return Errors::NotConfigured(MoveTemp(Error));
	case EOS_EResult::EOS_NotImplemented:		return Errors::NotImplemented(MoveTemp(Error));
	case EOS_EResult::EOS_Canceled:				return Errors::Cancelled(MoveTemp(Error));
	case EOS_EResult::EOS_NotFound:				return Errors::NotFound(MoveTemp(Error));
	case EOS_EResult::EOS_OperationWillRetry:	return Errors::WillRetry(MoveTemp(Error));
	case EOS_EResult::EOS_NoChange:				return Errors::NoChange(MoveTemp(Error));
	case EOS_EResult::EOS_VersionMismatch:		return Errors::IncompatibleVersion(MoveTemp(Error));
	case EOS_EResult::EOS_LimitExceeded:		return Errors::TooManyRequests(MoveTemp(Error));
	case EOS_EResult::EOS_TimedOut:				return Errors::Timeout(MoveTemp(Error));
	default:									return MoveTemp(Error);
	}
}

ErrorCodeType ErrorCodeFromEOSResult(EOS_EResult EosResult)
{
	return Errors::ErrorCode::Create(Errors::ErrorCode::Category::EOS_System, Errors::ErrorCode::Category::EOS, uint32(EosResult));
}

FOnlineError FromEOSResult(EOS_EResult EosResult, FErrorMapperEosFn&& MapperFn)
{
	return MapperFn(FOnlineError(ErrorCodeFromEOSResult(EosResult), FOnlineErrorDetailsEOS::Get()), EosResult);
}

} /* namespace UE::Online::Errors */
