// Copyright Epic Games, Inc. All Rights Reserved.

#include "Online/OnlineServicesEpicCommon.h"

#include "Online/OnlineServicesEpicCommonPlatformFactory.h"
#include "Online/OnlineServicesLog.h"

namespace UE::Online {

#define UE_ONLINE_EOS_KEY_NAME_FAST_TICK_LOCK TEXT("FastTickLock")

struct FOnlineServicesEpicCommonConfig
{
	bool bEnableAsyncOpFastTick = true;
};

namespace Meta {
	BEGIN_ONLINE_STRUCT_META(FOnlineServicesEpicCommonConfig)
	ONLINE_STRUCT_FIELD(FOnlineServicesEpicCommonConfig, bEnableAsyncOpFastTick)
	END_ONLINE_STRUCT_META()
}

FOnlineServicesEpicCommon::FOnlineServicesEpicCommon(const FString& InServiceConfigName, FName InInstanceName, FName InInstanceConfigName)
	: Super(InServiceConfigName, InInstanceName, InInstanceConfigName)
{
}

bool FOnlineServicesEpicCommon::PreInit()
{
	FOnlineServicesEpicCommonPlatformFactory& PlatformFactory = FOnlineServicesEpicCommonPlatformFactory::Get();
	EOSPlatformHandle = PlatformFactory.CreatePlatform(InstanceName, InstanceConfigName);
	
	if (!EOSPlatformHandle)
	{
		UE_LOGF(LogOnlineServices, Warning, "[%s] InstanceName=%ls InstanceConfigName=%ls EOSPlatformHandle=nullptr.", __FUNCTION__, *InstanceName.ToString(), *InstanceConfigName.ToString());
		return false;
	}

	return true;
}

void FOnlineServicesEpicCommon::UpdateConfig()
{
	Super::UpdateConfig();

	FOnlineServicesEpicCommonConfig Config;
	LoadConfig(Config);
	bEnableAsyncOpFastTick = Config.bEnableAsyncOpFastTick;
}

void FOnlineServicesEpicCommon::AddEOSSDKFastTick(FOnlineAsyncOp& InAsyncOp)
{
	if (bEnableAsyncOpFastTick)
	{
		if (EOSPlatformHandle)
		{
			TSharedPtr<IEOSFastTickLock> FastTickLock = EOSPlatformHandle->GetFastTickLock();
			InAsyncOp.Data.Set<TSharedPtr<IEOSFastTickLock>>(UE_ONLINE_EOS_KEY_NAME_FAST_TICK_LOCK, FastTickLock);
		}
	}
}

void FOnlineServicesEpicCommon::RemoveEOSSDKFastTick(FOnlineAsyncOp& InAsyncOp)
{
	if (TSharedPtr<IEOSFastTickLock>* FastTickLock = InAsyncOp.Data.Get<TSharedPtr<IEOSFastTickLock>>(UE_ONLINE_EOS_KEY_NAME_FAST_TICK_LOCK))
	{
		// Simply reset the shared pointer. A null shared pointer may remain on the async op.
		FastTickLock->Reset();
	}
}

void FOnlineServicesEpicCommon::WarnIfEncryptionKeyMissing(const FString& InterfaceName) const
{
	if (IEOSSDKManager* Manager = IEOSSDKManager::Get())
	{
		const FString& PlatformConfigName = GetEOSPlatformHandle()->GetConfigName();
		if (const FEOSSDKPlatformConfig* Config = Manager->GetPlatformConfig(PlatformConfigName))
		{
			const FString& EncryptionKey = Config->EncryptionKey;
			if (EncryptionKey.IsEmpty())
			{
				UE_LOGF(LogOnlineServices, Verbose, "%ls interface not available due to missing ClientEncryptionKey in config.", *InterfaceName);
			}
			else
			{
				// If we have an encryption key and still can't get the interface, something weird is going on.
				UE_LOGF(LogOnlineServices, Warning, "%ls interface not available despite encryption key being present.", *InterfaceName);
			}
		}
	}
}

void FOnlineServicesEpicCommon::FlushTick(float DeltaSeconds)
{
	Super::FlushTick(DeltaSeconds);

	GetEOSPlatformHandle()->Tick();
}

EOS_OnlinePlatformType EOnlinePlatformType_To_EOS_OnlinePlatformType(const EOnlinePlatformType& InType)
{
	switch (InType)
	{
	case EOnlinePlatformType::Epic:		return EOS_OPT_Epic;
	case EOnlinePlatformType::Steam:	return EOS_OPT_Steam;
	case EOnlinePlatformType::PSN:		return EOS_OPT_PSN;
	case EOnlinePlatformType::Nintendo:	return EOS_OPT_Nintendo;
	case EOnlinePlatformType::XBL:		return EOS_OPT_XBL;
	default:							checkNoEntry(); // Intentional fall-through
	case EOnlinePlatformType::Unknown:	return EOS_OPT_Unknown;
	}
}

EOnlinePlatformType EOS_OnlinePlatformType_To_EOnlinePlatformType(const EOS_OnlinePlatformType& InType)
{
	if (InType == EOS_OPT_Epic)
	{
		return EOnlinePlatformType::Epic;
	}
	else if (InType == EOS_OPT_Steam)
	{
		return EOnlinePlatformType::Steam;
	}
	else if (InType == EOS_OPT_PSN)
	{
		return EOnlinePlatformType::PSN;
	}
	else if (InType == EOS_OPT_Nintendo)
	{
		return EOnlinePlatformType::Nintendo;
	}
	else if (InType == EOS_OPT_XBL)
	{
		return EOnlinePlatformType::XBL;
	}
	else if (InType == EOS_OPT_Unknown)
	{
		return EOnlinePlatformType::Unknown;
	}
	else
	{
		checkNoEntry();
		return EOnlinePlatformType::Unknown;
	}
}

/* UE::Online */ }
