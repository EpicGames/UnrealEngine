// Copyright Epic Games, Inc. All Rights Reserved.

#include "Online/OnlineServicesEpicCommonModule.h"

#include "Modules/ModuleManager.h"
#include "Online/OnlineServicesEpicCommon.h"
#include "Online/OnlineServicesEpicCommonPlatformFactory.h"
#include "Online/OnlineServicesRegistry.h"

namespace UE::Online
{

void FOnlineServicesEpicCommonModule::StartupModule()
{
	FModuleManager::Get().LoadModuleChecked(TEXT("OnlineServicesInterface"));
	FModuleManager::Get().LoadModuleChecked(TEXT("EOSShared"));

	// Initialize the platform factory on startup.  This is necessary for the SDK to bind to rendering and input very early.
	FOnlineServicesEpicCommonPlatformFactory::Get();
}

void FOnlineServicesEpicCommonModule::ShutdownModule()
{
	UE::Online::FOnlineServicesEpicCommonPlatformFactory::TearDown();
}

/* UE::Online */ }

IMPLEMENT_MODULE(UE::Online::FOnlineServicesEpicCommonModule, OnlineServicesEpicCommon);
