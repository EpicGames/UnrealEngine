// Copyright Epic Games, Inc. All Rights Reserved.

#include "Online/OnlineServicesEpicCommonPlatformFactory.h"

#include "EOSShared.h"
#include "Online/OnlineExecHandler.h"
#include "Online/OnlineServicesEpicCommon.h"

#include "Misc/CommandLine.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/Fork.h"
#include "Misc/LazySingleton.h"
#include "Modules/ModuleManager.h"

#include "Online/OnlineServicesLog.h"
#include <eos_base.h>

namespace UE::Online {

FOnlineServicesEpicCommonPlatformFactory& FOnlineServicesEpicCommonPlatformFactory::Get()
{
	return TLazySingleton<FOnlineServicesEpicCommonPlatformFactory>::Get();
}

void FOnlineServicesEpicCommonPlatformFactory::TearDown()
{
	return TLazySingleton<FOnlineServicesEpicCommonPlatformFactory>::TearDown();
}

IEOSPlatformHandlePtr FOnlineServicesEpicCommonPlatformFactory::CreatePlatform(FName InstanceName, FName InstanceConfigName)
{
	const FName EOSSharedModuleName = TEXT("EOSShared");
	if (!FModuleManager::Get().IsModuleLoaded(EOSSharedModuleName))
	{
		FModuleManager::Get().LoadModuleChecked(EOSSharedModuleName);
	}

	IEOSSDKManager* const SDKManager = IEOSSDKManager::Get();
	if (!SDKManager)
	{
		UE_LOGF(LogOnlineServices, Error, "[FOnlineServicesEpicCommonPlatformFactory::CreatePlatform] EOSSDK has not been loaded.");
		return {};
	}

	if (!SDKManager->IsInitialized())
	{
		UE_LOGF(LogOnlineServices, Error, "[FOnlineServicesEpicCommonPlatformFactory::CreatePlatform] EOSSDK has not been initialized.");
		return {};
	}

	FString PlatformConfigName;
	if (!InstanceConfigName.IsNone())
	{
		PlatformConfigName = InstanceConfigName.ToString();
	}
	if (PlatformConfigName.IsEmpty())
	{
		PlatformConfigName = LoadEOSPlatformConfig(SDKManager);
	}
	if (PlatformConfigName.IsEmpty())
	{
		// Check for default platform config that other modules may have setup.
		PlatformConfigName = SDKManager->GetDefaultPlatformConfigName();
		if (PlatformConfigName.IsEmpty())
		{
			UE_LOGF(LogOnlineServices, Verbose, "[FOnlineServicesEpicCommonPlatformFactory::CreatePlatform] Could not find platform config.");
			return {};
		}
	}

	IEOSPlatformHandlePtr EOSPlatformHandle = SDKManager->CreatePlatform(PlatformConfigName, InstanceName);
	if (!EOSPlatformHandle)
	{
		UE_LOGF(LogOnlineServices, Warning, "[FOnlineServicesEpicCommonPlatformFactory::CreatePlatform] Failed to create platform.");
	}

	return EOSPlatformHandle;
}

FString FOnlineServicesEpicCommonPlatformFactory::LoadEOSPlatformConfig(IEOSSDKManager* const SDKManager)
{
	// TODO: OnlineServices.EOS hardcoded until FURL support is added to the Services EpicGame and EpicAccount
	FString ConfigSectionName = FString::Printf(TEXT("OnlineServices.EOS"));
	if (!GConfig->DoesSectionExist(*ConfigSectionName, GEngineIni))
	{
		return FString();
	}

	// Check for explicit shared name to use.
	FString PlatformConfigName;
	GConfig->GetString(*ConfigSectionName, TEXT("PlatformConfigName"), PlatformConfigName, GEngineIni);
	if (!PlatformConfigName.IsEmpty())
	{
		return PlatformConfigName;
	}

	// Check for cached config.
	if (SDKManager->GetPlatformConfig(ConfigSectionName))
	{
		return ConfigSectionName;
	}

	// Check for legacy config. This should be handled by EOSShared instead, see FEOSSDKManager::GetPlatformConfig for details.
	FEOSSDKPlatformConfig PlatformConfig;
	PlatformConfig.Name = ConfigSectionName;

	GConfig->GetString(*ConfigSectionName, TEXT("ProductId"), PlatformConfig.ProductId, GEngineIni);
	if (PlatformConfig.ProductId.IsEmpty())
	{
		// If we're missing ProductId, assume we're missing the rest and instead rely on default EOSShared config.
		return FString();
	}

	// Instead of specifying this config under the OnlineServices.EOS section, options should be moved to a new EOSSDK.Platform.<name> section so
	// all modules relying on the EOSSDK can share the same config and platform instance. See FEOSSDKManager::GetPlatformConfig for details.
	UE_LOGF(LogOnlineServices, Warning, "[LoadEOSPlatformConfig] Using legacy config from %ls, use EOSShared named config instead.", *ConfigSectionName);

	GConfig->GetString(*ConfigSectionName, TEXT("SandboxId"), PlatformConfig.SandboxId, GEngineIni);
	GConfig->GetString(*ConfigSectionName, TEXT("DeploymentId"), PlatformConfig.DeploymentId, GEngineIni);
	GConfig->GetString(*ConfigSectionName, TEXT("ClientId"), PlatformConfig.ClientId, GEngineIni);
	GConfig->GetString(*ConfigSectionName, TEXT("ClientSecret"), PlatformConfig.ClientSecret, GEngineIni);
	// Config key renamed to ClientEncryptionKey as EncryptionKey gets removed from packaged builds due to IniKeyDenylist=EncryptionKey entry in BaseGame.ini.
	GConfig->GetString(*ConfigSectionName, TEXT("ClientEncryptionKey"), PlatformConfig.EncryptionKey, GEngineIni);

	const FString CacheDirBase = SDKManager->GetCacheDirBase();
	PlatformConfig.CacheDirectory = CacheDirBase.IsEmpty() ? FString() : CacheDirBase / TEXT("OnlineServicesEOS");

	PlatformConfig.bIsServer = IsRunningDedicatedServer() ? EOS_TRUE : EOS_FALSE;
	if (!IsRunningGame())
	{
		PlatformConfig.bLoadingInEditor = true;
	}
	else
	{
		PlatformConfig.bWindowsEnableOverlayD3D9 = true;
		PlatformConfig.bWindowsEnableOverlayD3D10 = true;
		PlatformConfig.bWindowsEnableOverlayOpenGL = true;
	}

	// After we have loaded the platform config, we'll check to see if there is any command-line overrides present

	FString SandboxIdOverride;
	// Get the -epicsandboxid argument. This generally comes from EGS.
	bool bHasSandboxIdOverride = FParse::Value(FCommandLine::Get(), TEXT("EpicSandboxId="), SandboxIdOverride);
	// Prefer -EpicSandboxIdOverride over previous.
	bHasSandboxIdOverride |= FParse::Value(FCommandLine::Get(), TEXT("EpicSandboxIdOverride="), SandboxIdOverride);

	if (bHasSandboxIdOverride)
	{
		PlatformConfig.SandboxId = SandboxIdOverride;
	}

	FString DeploymentIdOverride;
	// Get the -epicdeploymentid argument. This generally comes from EGS.
	bool bHasDeploymentIdOverride = FParse::Value(FCommandLine::Get(), TEXT("EpicDeploymentId="), DeploymentIdOverride);
	// Prefer -EpicDeploymentIdOverride over previous.
	bHasDeploymentIdOverride |= FParse::Value(FCommandLine::Get(), TEXT("EpicDeploymentIdOverride="), DeploymentIdOverride);

	if (bHasDeploymentIdOverride)
	{
		PlatformConfig.DeploymentId = DeploymentIdOverride;
	}

	if (!SDKManager->AddPlatformConfig(PlatformConfig))
	{
		return FString();
	}

	return ConfigSectionName;
}

/* UE::Online */ }
