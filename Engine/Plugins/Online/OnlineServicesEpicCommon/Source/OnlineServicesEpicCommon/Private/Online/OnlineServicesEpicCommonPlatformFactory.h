// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"

#include "IEOSSDKManager.h"

using IEOSPlatformHandlePtr = TSharedPtr<class IEOSPlatformHandle>;
class FLazySingleton;

namespace UE::Online
{

	/** Factory class to create EOS Platforms for online services */
	class FOnlineServicesEpicCommonPlatformFactory
	{
	public:
		/**
		 * Get the platform factory
		 * @return the platform factory singleton
		 */
		static FOnlineServicesEpicCommonPlatformFactory& Get();
		/**
		 * Tear down the singleton instance. This only cleans up the singleton and has no impact on any platform handles created by this (aside from DefaultEOSPlatformHandle's ref count decreasing).
		 */
		static void TearDown();

		/**
		 * Create a new platform instance for the given Instance/Config pair.
		 * Note If InstanceConfigName = NAME_None, this will attempt to resolve a config to use from various sources, including OnlineServices config, and EOSSDKManager cached configs.
		 * @param InstanceName the "instance", typically a WorldContextHandle
		 * @param InstanceConfigName the named EOS platform config to use
		 * @return a new platform, or null on failure
		 */
		IEOSPlatformHandlePtr CreatePlatform(FName InstanceName, FName InstanceConfigName);

	private:
		FOnlineServicesEpicCommonPlatformFactory() = default;

		FString LoadEOSPlatformConfig(IEOSSDKManager* const SDKManager);

		friend FLazySingleton;
	};

	/* UE::Online */
}
