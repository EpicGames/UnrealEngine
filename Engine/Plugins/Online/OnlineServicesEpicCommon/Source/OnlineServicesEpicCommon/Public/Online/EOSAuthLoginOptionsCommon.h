// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Online/Auth.h"
#include "Online/OnlineAsyncOp.h"

#if defined(EOS_PLATFORM_BASE_FILE_NAME)
#include EOS_PLATFORM_BASE_FILE_NAME
#endif

#include "eos_auth_types.h"

namespace UE::Online {

enum class EEOSAuthTranslationFlags : uint8
{
	None = 0,
	SetId = 1 << 0,
	SetTokenFromString = 1 << 1,
	SetTokenFromExternalAuth = 1 << 2,
};
ENUM_CLASS_FLAGS(EEOSAuthTranslationFlags);

struct FEOSAuthTranslationTraits
{
	EOS_ELoginCredentialType Type = EOS_ELoginCredentialType::EOS_LCT_Password;
	EEOSAuthTranslationFlags Flags = EEOSAuthTranslationFlags::None;
};

struct FEOSExternalAuthTranslationTraits
{
	EOS_EExternalCredentialType Type = EOS_EExternalCredentialType::EOS_ECT_EPIC;
	EOS_ELinkAccountFlags LinkAccountFlags = EOS_ELinkAccountFlags::EOS_LA_NoFlags;
};

class FEOSAuthLoginOptionsCommon : public EOS_Auth_LoginOptions
{
public:
	FEOSAuthLoginOptionsCommon(const FEOSAuthLoginOptionsCommon&) = delete;
	FEOSAuthLoginOptionsCommon& operator=(const FEOSAuthLoginOptionsCommon&) = delete;
	ONLINESERVICESEPICCOMMON_API FEOSAuthLoginOptionsCommon(FEOSAuthLoginOptionsCommon&&);
	ONLINESERVICESEPICCOMMON_API FEOSAuthLoginOptionsCommon& operator=(FEOSAuthLoginOptionsCommon&&);

	EOS_ELinkAccountFlags GetLinkAccountFlags() const { return LinkAccountFlags; }

protected:
	ONLINESERVICESEPICCOMMON_API FEOSAuthLoginOptionsCommon();

	static ONLINESERVICESEPICCOMMON_API const FEOSAuthTranslationTraits* GetLoginTranslatorTraits(FName Name);
	static ONLINESERVICESEPICCOMMON_API const FEOSExternalAuthTranslationTraits* GetExternalAuthTranslationTraits(FName ExternalAuthType);

	static bool InitSystemAuthCredentialOptions(FEOSAuthLoginOptionsCommon& Options) { return true; }

	EOS_Auth_Credentials CredentialsData;
	EOS_ELinkAccountFlags LinkAccountFlags = EOS_ELinkAccountFlags::EOS_LA_NoFlags;
	TArray<char> IdUtf8;
	TArray<char> TokenUtf8;
};

}
