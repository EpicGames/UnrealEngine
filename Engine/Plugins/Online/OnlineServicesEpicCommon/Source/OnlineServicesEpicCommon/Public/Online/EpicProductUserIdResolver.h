// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Async/Future.h"
#include "EOSShared.h"
#include "Online/Auth.h"
#include "Online/CoreOnline.h"

namespace UE::Online
{

/** Exists to provide components that are instantiated in both OnlineServicesEOSGS and OnlineServicesEpicGame a resolve path that works in both. */
class IEpicProductUserIdResolver
{
public:
	virtual TFuture<FAccountId> ResolveAccountId(const FAccountId LocalAccountId, const EOS_ProductUserId ProductUserId) = 0;
	virtual TFuture<TArray<FAccountId>> ResolveAccountIds(const FAccountId LocalAccountId, const TArray<EOS_ProductUserId>& ProductUserIds) = 0;

	ONLINESERVICESEPICCOMMON_API TFunction<TFuture<FAccountId>(class FOnlineAsyncOp& InAsyncOp, const EOS_ProductUserId ProductUserId)> ResolveProductUserIdFn();
	ONLINESERVICESEPICCOMMON_API TFunction<TFuture<TArray<FAccountId>>(class FOnlineAsyncOp& InAsyncOp, const TArray<EOS_ProductUserId>& ProductUserIds)> ResolveProductUserIdsFn();
};

} // namespace UE::Online