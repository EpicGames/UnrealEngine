// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EOSShared.h"
#include "Online/OnlineError.h"

/*
 *	Proper usage:
 *		Errors::FromEOSResult(EOSResult::EOS_PlayerDataStorage_FileSizeTooLarge);
 *	Certain EOS errors are predefined to have a common error parent type (see Internal_EOSWrapInner). i.e.
 *		(Errors::FromEOSResult(EOSResult::EOS_NoConnection) == Errors::NoConnection()) == true
 */

namespace UE::Online::Errors {

	UE_ONLINE_ERROR_CATEGORY(EOS, ThirdPartyPlugin, 0x4, "EOS")

	typedef TFunction<FOnlineError(FOnlineError&& Error, EOS_EResult Result)> FErrorMapperEosFn;

	ONLINESERVICESEPICCOMMON_API FOnlineError MapCommonEOSError(FOnlineError&& Error, EOS_EResult Result);
	ONLINESERVICESEPICCOMMON_API ErrorCodeType ErrorCodeFromEOSResult(EOS_EResult Result);
	ONLINESERVICESEPICCOMMON_API FOnlineError FromEOSResult(EOS_EResult Result, FErrorMapperEosFn&& MapperFn = &MapCommonEOSError);

/* UE::Online::Errors */ }

// These are extern'd out of the namespace for some of the Catch API that can't see these operators if they are inside
inline bool operator==(const UE::Online::FOnlineError& OnlineError, EOS_EResult EosResult)
{
	return OnlineError == UE::Online::Errors::ErrorCodeFromEOSResult(EosResult);
}

inline bool operator==(EOS_EResult EosResult, const UE::Online::FOnlineError& OnlineError)
{
	return OnlineError == EosResult;
}

inline bool operator!=(const UE::Online::FOnlineError& OnlineError, EOS_EResult EosResult)
{
	return !(OnlineError == EosResult);
}

inline bool operator!=(EOS_EResult EosResult, const UE::Online::FOnlineError& OnlineError)
{
	return OnlineError != EosResult;
}
