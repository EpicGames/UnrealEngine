// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Online/OnlineServicesCommon.h"

#if defined(EOS_PLATFORM_BASE_FILE_NAME)
#include EOS_PLATFORM_BASE_FILE_NAME
#endif
#include "eos_common.h"

#define UE_API ONLINESERVICESEPICCOMMON_API

using IEOSPlatformHandlePtr = TSharedPtr<class IEOSPlatformHandle>;

namespace UE::Online {

class FOnlineServicesEpicCommon : public FOnlineServicesCommon
{
public:
	using Super = FOnlineServicesCommon;

	UE_API FOnlineServicesEpicCommon(const FString& InServiceConfigName, FName InInstanceName, FName InInstanceConfigName);
	virtual ~FOnlineServicesEpicCommon() = default;

	UE_API virtual bool PreInit();
	UE_API virtual void UpdateConfig() override;

	IEOSPlatformHandlePtr GetEOSPlatformHandle() const { return EOSPlatformHandle; }

	/** Enable EOSSDK to tick as fast as it can while this operation is outstanding. Expected usage is to call this before calling an SDK function. */
	UE_API void AddEOSSDKFastTick(FOnlineAsyncOp& InAsyncOp);
	/** Remove fast ticking of EOSSDK for an operation. Expected usage is to call this after an SDK function's completion delegate triggers. If this is not called, the fast tick will be removed when the operation destructs. */
	UE_API void RemoveEOSSDKFastTick(FOnlineAsyncOp& InAsyncOp);

protected:
	UE_API void WarnIfEncryptionKeyMissing(const FString& InterfaceName) const;
	
	UE_API virtual void FlushTick(float DeltaSeconds) override;

	IEOSPlatformHandlePtr EOSPlatformHandle;

	bool bEnableAsyncOpFastTick = true;
};

UE_API EOS_OnlinePlatformType EOnlinePlatformType_To_EOS_OnlinePlatformType(const EOnlinePlatformType& InType);
UE_API EOnlinePlatformType EOS_OnlinePlatformType_To_EOnlinePlatformType(const EOS_OnlinePlatformType& InType);

/* UE::Online */ }

#undef UE_API
